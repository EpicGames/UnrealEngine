[/Script/DisplayClusterLaunchEditor.DisplayClusterLaunchEditorProjectSettings]
AdditionalConsoleVariables=("p.Chaos.Solver.Deterministic=1","LevelInstance.ForceEditorWorldMode=1")
AdditionalConsoleCommands=("DisableAllScreenMessages")
CommandLineArguments=("nomcp","messaging","dc_cluster","nosplash","fixedseed","NoVerifyGC","noxrstereo","xrtrackingonly","RemoteControlIsHeadless","dx12","dc_dev_mono","unattended","handleensurepercent=0","ini:Engine:[/Script/Engine.Engine]:GameEngine=/Script/DisplayCluster.DisplayClusterGameEngine,[/Script/Engine.Engine]:GameViewportClientClassName=/Script/DisplayCluster.DisplayClusterViewportClient,[/Script/Engine.UserInterfaceSettings]:bAllowHighDPIInGameMode=True,[/Script/Engine.UserInterfaceSettings]:bAllowHighDpiWhenUnattended=True","ini:Game:[/Script/EngineSettings.GeneralProjectSettings]:bUseBorderlessWindow=True")
Logging=(Category="LogDisplayClusterRenderSync",VerbosityLevel=Log)
