{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "nDisplay Launch",
	"Description": "Launch local nDisplay nodes with ease.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": false,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Plugins":
	[
		{
			"Name": "nDisplay",
			"Enabled": true
		},
		{
			"Name": "ConsoleVariables",
			"Enabled": true
		},
		{
			"Name": "ConcertMain",
			"Enabled": true 
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "DisplayClusterLaunchEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": ["Win64", "Linux"]
		}
	]
}
