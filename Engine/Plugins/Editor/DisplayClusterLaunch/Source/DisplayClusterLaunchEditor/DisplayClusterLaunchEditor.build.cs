// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class DisplayClusterLaunchEditor : ModuleRules
{
	public DisplayClusterLaunchEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"OutputLog", 
				"PlacementMode"
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"Concert",
				"ConcertClient",
				"ConcertTransport",
				"CoreUObject",
				"ConcertSyncClient",
				"ConsoleVariablesEditor",
				"ConsoleVariablesEditorRuntime",
				"ContentBrowser",
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"Engine",
				"EditorStyle",
				"EditorWidgets",
				"InputCore",
				"Kismet",
				"Messaging",
				"MultiUserClient",
				"Networking",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus", 
				"ToolWidgets",
				"UdpMessaging",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
