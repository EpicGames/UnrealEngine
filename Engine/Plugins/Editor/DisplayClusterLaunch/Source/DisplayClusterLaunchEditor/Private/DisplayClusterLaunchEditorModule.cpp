// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLaunchEditorModule.h"
#include "DisplayClusterLaunchEditorLog.h"
#include "DisplayClusterLaunchEditorProjectSettings.h"
#include "DisplayClusterLaunchEditorStyle.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "DisplayClusterRootActor.h"
#include "Framework/Application/SlateApplication.h"
#include "IDisplayClusterConfiguration.h"
#include "DisplayClusterConfigurationTypes.h"

#include "ConcertSettings.h"
#include "ConcertClientSettings.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "IMultiUserClientModule.h"

#include "Shared/UdpMessagingSettings.h"

#include "Algo/RemoveIf.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "IPlacementModeModule.h"
#include "Engine/GameEngine.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "Misc/ConfigCacheIni.h"
 
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "ConsoleVariablesAsset.h"
#include "Interfaces/IPluginManager.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterLaunchEditorModule"

FString EnumToString(const FString EnumName, const int32 EnumValue)
{
	const FString EnumPath = "/Script/DisplayClusterLaunchEditor." + EnumName;
	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPath, EFindObjectFlags::ExactClass);
	
	if (!EnumPtr)
	{
		return LOCTEXT("EnumNotFound", "Enum not found").ToString();
	}
	
	return EnumPtr->GetNameStringByIndex(EnumValue);
}

static UWorld* GetCurrentWorld()
{
	UWorld* CurrentWorld = nullptr;
	if (GIsEditor)
	{
		CurrentWorld = GEditor->GetEditorWorldContext().World();
	}
	else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		CurrentWorld = GameEngine->GetGameWorld();
	}
	return CurrentWorld;
}

FDisplayClusterLaunchEditorModule& FDisplayClusterLaunchEditorModule::Get()
{
	return FModuleManager::GetModuleChecked<FDisplayClusterLaunchEditorModule>("DisplayClusterLaunchEditor");
}

void FDisplayClusterLaunchEditorModule::StartupModule()
{
	FDisplayClusterLaunchEditorStyle::Initialize();
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FDisplayClusterLaunchEditorModule::OnFEngineLoopInitComplete);
}

void FDisplayClusterLaunchEditorModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
	FDisplayClusterLaunchEditorStyle::Shutdown();
	
	// Unregister project settings
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		SettingsModule.UnregisterSettings("Project", "Plugins", "nDisplay Launch");
	}
	StopMultiUserDiscovery();
}

void FDisplayClusterLaunchEditorModule::EnsureMultiUserDiscovery()
{
	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		if (!bDidStartDiscovery)
		{
			const IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			ConcertClient->StartDiscovery();
			bDidStartDiscovery = true;
		}
	}
	UE_CLOGF(!bDidStartDiscovery, LogDisplayClusterLaunchEditor, Error, "Unable to start Multi-user discovery process.");
}

void FDisplayClusterLaunchEditorModule::StopMultiUserDiscovery()
{
	if (IConcertSyncClientModule* ConcertSyncClientModule = (IConcertSyncClientModule*)FModuleManager::Get().GetModule("ConcertSyncClient"))
	{
		if (const TSharedPtr<IConcertSyncClient> ConcertSyncClient = ConcertSyncClientModule->GetClient(TEXT("MultiUser")))
		{
			const IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			// Concert may close all existing discovery requests so we have to check to see if we are still have discovery enabled before
			// attempting to stop.
			if (bDidStartDiscovery && ConcertClient->IsDiscoveryEnabled())
			{
				ConcertClient->StopDiscovery();
			}
			ConcertClient->OnKnownServersUpdated().RemoveAll(this);
			bDidStartDiscovery = false;
		}
	}
}

void FDisplayClusterLaunchEditorModule::OpenProjectSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
		.ShowViewer("Project", "Plugins", "nDisplay Launch");
}

bool AddUdpMessagingArguments(FString& ConcatenatedArguments)
{
	// Get from reflection because UUdpMessagingSettings does not export its API
	FConfigFile* EngineConfig = GConfig ? GConfig->FindConfigFileWithBaseName(FName(TEXT("Engine"))) : nullptr;
	if (EngineConfig)
	{
		TArray<FString> Settings;
		FString Setting;
		// Unicast endpoint setting
		EngineConfig->GetString(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("UnicastEndpoint"), Setting);
		// if the unicast endpoint port is bound, concatenate it
		if (Setting != "0.0.0.0:0" && !Setting.IsEmpty())
		{
			ConcatenatedArguments += TEXT(" -UDPMESSAGING_TRANSPORT_UNICAST=") + Setting;
		}
		// Multicast endpoint setting
		EngineConfig->GetString(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("MulticastEndpoint"), Setting);
		ConcatenatedArguments += TEXT(" -UDPMESSAGING_TRANSPORT_MULTICAST=") + Setting;
		// Static endpoints setting
		Settings.Empty(1);
		EngineConfig->GetArray(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("StaticEndpoints"), Settings);
		if (Settings.Num() > 0)
		{
			ConcatenatedArguments += TEXT(" -UDPMESSAGING_TRANSPORT_STATIC=");
			ConcatenatedArguments += Settings[0];
			for (int32 i = 1; i < Settings.Num(); ++i)
			{
				ConcatenatedArguments += ',';
				ConcatenatedArguments += Settings[i];
			}
		}
		return true;
	}
	return false;
}

FString AppendRandomNumbersToString(const FString InString, uint8 NumberToAppend = 6)
{
	FString RandomizedString = "_";
	for (uint8 RandomIteration = 0; RandomIteration < NumberToAppend; RandomIteration++)
	{
		RandomizedString += FString::FromInt(FMath::RandRange(0, 9));
	}
	return InString + RandomizedString;
}
 
FString GetConcertArguments(const FString& ServerName, const FString& SessionName)
{
	const UConcertClientConfig* ConcertClientConfig = GetDefault<UConcertClientConfig>();
	ensureAlwaysMsgf(ConcertClientConfig, TEXT("%hs: Unable to launch nDisplay because there is no UConcertClientConfig object."), __FUNCTION__);
	FString ReturnValue =
		FString::Printf(TEXT("-CONCERTISHEADLESS -CONCERTRETRYAUTOCONNECTONERROR -CONCERTAUTOCONNECT -CONCERTSERVER=\"%s\" -CONCERTSESSION=\"%s\""),
			 *ServerName, *SessionName);
	return ReturnValue;
}

const FString GetInputArguments()
{
	const FString ReturnValue = FString::Printf(TEXT("-ini:Input:[/Script/Engine.InputSettings]:DefaultPlayerInputClass=/Script/DisplayCluster.DisplayClusterPlayerInput"));
	return ReturnValue;
}

void FDisplayClusterLaunchEditorModule::LaunchConcertServer()
{
	IMultiUserClientModule& MultiUserClientModule = IMultiUserClientModule::Get();
	{
		FServerLaunchOverrides Overrides;
		Overrides.ServerName = GetConcertServerName();
		ConcertServerRequestStatus = EConcertServerRequestStatus::LaunchRequested;
		TOptional<FProcHandle> ServerHandle = MultiUserClientModule.LaunchConcertServer(Overrides);
		if (ServerHandle.IsSet() && ServerHandle.GetValue().IsValid())
		{
			ServerTrackingData.MultiUserServerHandle = ServerHandle.GetValue();
		}
	}
}

void FDisplayClusterLaunchEditorModule::FindOrLaunchConcertServer()
{
	// Ensure we have the client, otherwise we can't do anything
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	if (!ConcertSyncClient)
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: The ConcertSyncClient could not be found. Please check the output log for errors and try again.", __FUNCTION__);
		return;
	}

	const IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
	const TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession();
	bool bIsConnectedToSession = ConcertClientSession.IsValid();
	ConcertClient->OnKnownServersUpdated().RemoveAll(this);

	// Shutdown existing server no matter what because we need to hook into OnServersAssumedReady
	IMultiUserClientModule& MultiUserClientModule = IMultiUserClientModule::Get();

	TArray<FConcertServerInfo> Servers = ConcertClient->GetKnownServers();
	if (FPlatformProcess::IsProcRunning(ServerTrackingData.MultiUserServerHandle))
	{
		// Try to reuse last server
		FConcertServerInfo* Server = Algo::FindByPredicate(Servers, [Name = GetConcertServerName()](const FConcertServerInfo& Server)
			{
				return Server.ServerName == Name;
			});
		if (Server)
		{
			ConcertServerRequestStatus = EConcertServerRequestStatus::ReuseExisting;
			OnServersAssumedReady();
		}
		else
		{
			// We are waiting for servers to register with Concert.
			ConcertClient->OnKnownServersUpdated().AddRaw(
				this, &FDisplayClusterLaunchEditorModule::OnServersAssumedReady
				);
		}
	}
	else if (Servers.Num() > 0 && (MultiUserClientModule.IsConcertServerRunning() || bIsConnectedToSession))
	{
		ConcertServerRequestStatus = EConcertServerRequestStatus::ReuseExisting;
		OnServersAssumedReady();
	}
	else
	{
		if (Servers.Num() == 0)
		{
			ConcertServerRequestStatus = EConcertServerRequestStatus::ReuseExisting;
		}

		// We are waiting for servers to register with Concert.
		ConcertClient->OnKnownServersUpdated().AddRaw(
			this, &FDisplayClusterLaunchEditorModule::OnServersAssumedReady
			);
		if (!MultiUserClientModule.IsConcertServerRunning())
		{
			LaunchConcertServer();
		}
	}
}

void FDisplayClusterLaunchEditorModule::OnServersAssumedReady()
{
	if (ConcertServerRequestStatus == EConcertServerRequestStatus::ShutdownRequested)
	{
		// If this method was called when trying to shut down the previous server
		// then loop back so we can return after the new server is launched
		ConcertServerRequestStatus = EConcertServerRequestStatus::None;
		FindOrLaunchConcertServer();
	}
	else if (ConcertServerRequestStatus == EConcertServerRequestStatus::LaunchRequested ||
		ConcertServerRequestStatus == EConcertServerRequestStatus::ReuseExisting)
	{
		FindAppropriateServer();
	}
	else
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Warning, "%s: OnServersAssumedReady was called when ConcertServerRequestStatus was None.", __FUNCTION__);
	}
}

void FDisplayClusterLaunchEditorModule::FindAppropriateServer()
{
	ConcertServerRequestStatus = EConcertServerRequestStatus::None;
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	if (!ConcertSyncClient)
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: The ConcertSyncClient could not be found. Please check the output log for errors and try again.", __FUNCTION__);
	}


	const IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
	// Try to connect to an existing session even if we launched a new server
	TArray<FConcertServerInfo> Servers = ConcertClient->GetKnownServers();
	FConcertServerInfo* Server = nullptr;
	if (const TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
	{
		const FConcertSessionInfo& SessionInfo = ConcertClientSession->GetSessionInfo();
		Server = Algo::FindByPredicate(Servers, [&SessionInfo](const FConcertServerInfo& Server)
			{
				return Server.InstanceInfo.InstanceId == SessionInfo.ServerInstanceId;
			});
		// The list of known servers must contain the connected session.
		if (!Server)
		{
			UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: The list of known servers does not contain the connected session. Please check the output log for errors and try again.", __FUNCTION__);
			return;
		}
		CachedConcertSessionName = ConcertClientSession->GetName();
	}
	else
	{
		// If no session, we need to try to find a server with a name matching the cached name
		Server = Algo::FindByPredicate(Servers, [Name = GetConcertServerName()](const FConcertServerInfo& Server)
			{
				return Server.ServerName == Name;
			});
		if (!Server && Servers.Num() > 0)
		{
			// Pick a server to use.
			Server = &Servers[0];
		}
	}

	if (Server)
	{
		ServerTrackingData.MultiUserServerInfo = *Server;
		ServerTrackingData.GeneratedMultiUserServerName = Server->ServerName;
		ConnectToSession();
	}
	else
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: Servers exist but a matching server was not found. Try connecting to a server and session manually.", __FUNCTION__);
	}
}

void FDisplayClusterLaunchEditorModule::ConnectToSession()
{
	// Session Management
	// First check to see if we're in a session already
	if (const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		const IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		if (const TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
		{
			// If we're already connected, go straight into launch
			LaunchDisplayClusterProcess();
			return;
		}
		
		const UConcertClientConfig* CurrentConfig = ConcertClient->GetConfiguration();
		UConcertClientConfig* AutoConnectConfig = DuplicateObject(CurrentConfig, GetTransientPackage());

		AutoConnectConfig->bAutoConnect = true;
		AutoConnectConfig->bRetryAutoConnectOnError = true;
		AutoConnectConfig->DefaultServerURL = GetConcertServerName();
		AutoConnectConfig->DefaultSessionName = GetConcertSessionName();
		
		ConcertClient->Configure(AutoConnectConfig);
		if (!ConcertClient->IsConfigured())
		{
			UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: The ConcertSyncClient is not configured. Please check the output log for errors and try again.", __FUNCTION__);
			return;
		}
		
		// Initiate the auto connect to the named server and session.
		if (ConcertClient->CanAutoConnect())
		{
			ConcertClient->StartAutoConnect();
			LaunchDisplayClusterProcess();
		}
		else
		{
			ConcertClient->Configure(CurrentConfig);
			UE_LOGF(LogDisplayClusterLaunchEditor, Error, "Unable to start Multi-user auto connect routines.");
		}
	}
}

void FDisplayClusterLaunchEditorModule::TryLaunchDisplayClusterProcess()
{
	if (!ensureAlwaysMsgf(GetDefault<UDisplayClusterLaunchEditorProjectSettings>(), TEXT("%hs: Unable to launch nDisplay because there is no UDisplayClusterLaunchEditorProjectSettings object."), __FUNCTION__))
	{
		return;
	}
	
	RefreshSelectedDisplayActor();

	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> ConfigsInWorld = GetAllDisplayClusterConfigsInWorld();

	if (!DoesCurrentWorldHaveDisplayClusterConfig())
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: Unable to launch nDisplay because there are no valid nDisplay configurations in the world.", __FUNCTION__);
		return;
	}
	
	// Create Multi-user params async
	if (GetConnectToMultiUser())
	{
		EnsureMultiUserDiscovery();
		CachedConcertSessionName.Empty();
		FindOrLaunchConcertServer();
	}
	else
	{
		LaunchDisplayClusterProcess();
	}
}

void FDisplayClusterLaunchEditorModule::LaunchDisplayClusterProcess()
{		
	UE_LOGF(LogDisplayClusterLaunchEditor, Log, "%s: Launching nDisplay processes...", __FUNCTION__);

	// Open a modal to prompt for save, if dirty. Yes = Save & Continue. No = Continue Without Saving. Cancel = Stop Opening Assets.

	UPackage* PackageToSave = nullptr;

	if (UWorld* World = GetCurrentWorld())
	{
		if (ULevel* Level = World->GetCurrentLevel())
		{
			PackageToSave = Level->GetPackage();
		}
	}

	if (PackageToSave)
	{
		const FEditorFileUtils::EPromptReturnCode DialogueResponse =
				FEditorFileUtils::PromptForCheckoutAndSave(
					{PackageToSave},
					true,
					true,
					LOCTEXT("SavePackagesTitle", "Save Packages"),
					LOCTEXT("ConfirmOpenLevelFormat", "Do you want to save the current level?\n\nCancel to abort launch.\n")
				);

		if (DialogueResponse != FEditorFileUtils::EPromptReturnCode::PR_Success)
		{
			return;
		}
	}
	
	// If the level was renamed (e.g. if it was Untitled), the config actor soft path needs to be updated.
	RefreshSelectedDisplayActor();

	FString ConcertArguments;

	if (GetConnectToMultiUser())
	{
		ConcertArguments = GetConcertArguments(GetConcertServerName(), GetConcertSessionName());
	}

	const UDisplayClusterLaunchEditorProjectSettings* ProjectSettings = GetDefault<UDisplayClusterLaunchEditorProjectSettings>();
	UDisplayClusterConfigurationData* ConfigDataToUse = nullptr;
	FString ConfigActorPath;

	// If it's valid we need to check the selected nodes against the current config. If they don't exist, we need to get the first one.
	if (const ADisplayClusterRootActor* ConfigActor = Cast<ADisplayClusterRootActor>(SelectedDisplayClusterConfigActor.ResolveObject()))
	{
		// Duplicate existing config data so we can make non-destructive edits.
		UDisplayClusterBlueprint* ConfigBlueprint = CastChecked<UDisplayClusterBlueprint>(ConfigActor->GetClass()->ClassGeneratedBy);
		ConfigDataToUse = DuplicateObject(ConfigBlueprint->GetOrLoadConfig(), GetTransientPackage());

		ApplyDisplayClusterConfigOverrides(ConfigDataToUse);

		const FString FilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT(""), TEXT(".ndisplay"));
		if (!ensureAlways(IDisplayClusterConfiguration::Get().SaveConfig(ConfigDataToUse, FilePath)))
		{
			UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: Unable to launch nDisplay because the selected nDisplay Configuration could not be saved to a .ndisplay file. See the log for more information.", __FUNCTION__);
			return;
		}
	
		ConfigActorPath = FString::Printf(TEXT("-dc_cfg=\"%s\""), *FilePath);
	}
	else
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: Unable to launch nDisplay because the selected nDisplay Config Actor could not be resolved or does not exist in the current level.", __FUNCTION__);
		return;
	}

	const FString EditorBinary = FPlatformProcess::ExecutablePath();
	
	const FString Project = FPaths::SetExtension(FPaths::Combine(FPaths::ProjectDir(), FApp::GetProjectName()),".uproject");
	const FString Map = GetCurrentWorld()->GetCurrentLevel()->GetPackage()->GetFName().ToString();

	for (const FString& Node : SelectedDisplayClusterConfigActorNodes)
	{
		FString ConcatenatedCommandLineArguments;
		FString ConcatenatedConsoleCommands;
		FString ConcatenatedLogCommands;

		TSet<FString> DPCvars;

		// Fullscreen/Windowed
		if (TObjectPtr<UDisplayClusterConfigurationClusterNode>* NodePtrPtr = ConfigDataToUse->Cluster->Nodes.Find(Node))
		{
			if (const UDisplayClusterConfigurationClusterNode* NodePtr = *NodePtrPtr)
			{
				if (NodePtr->bIsFullscreen)
				{
					ConcatenatedCommandLineArguments += "-fullscreen ";
				}
				else
				{
					ConcatenatedCommandLineArguments +=
						FString::Printf(
							TEXT("-windowed -forceres -WinX=%i -WinY=%i, -ResX=%i, -ResY=%i "),
								NodePtr->WindowRect.X, NodePtr->WindowRect.Y,
								NodePtr->WindowRect.W, NodePtr->WindowRect.H
						);
				}

				// Headless
				if (NodePtr->bRenderHeadless)
				{
					ConcatenatedCommandLineArguments += "-RenderOffscreen ";
				}

				// Graphics adapter
				if (NodePtr->GraphicsAdapter >= 0)
				{
					DPCvars.Add(FString::Printf(TEXT("r.GraphicsAdapter=%d"), NodePtr->GraphicsAdapter));
				}
			}
		}
		// Unreal Insights support
		if (ProjectSettings->bEnableUnrealInsights)
		{
			// Enable trace
			ConcatenatedCommandLineArguments += " -trace";
			if (ProjectSettings->bEnableStatNamedEvents)
			{
				ConcatenatedCommandLineArguments += " -statnamedevents";
			}
			// Override save directory if desired
			if (!ProjectSettings->ExplicitTraceFileSaveDirectory.Path.IsEmpty())
			{
				FString TraceFilePath =
					ProjectSettings->ExplicitTraceFileSaveDirectory.Path / FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S"));
				ConcatenatedCommandLineArguments +=
					FString::Printf(TEXT(" -tracefile=%s "), *TraceFilePath);
			}
		}
	
		GetProjectSettingsArguments(ProjectSettings, ConcatenatedCommandLineArguments, ConcatenatedConsoleCommands, DPCvars,
					 ConcatenatedLogCommands);
		
		AddUdpMessagingArguments(ConcatenatedCommandLineArguments);
		// Add nDisplay node information
		ConcatenatedCommandLineArguments += " " + ConfigActorPath;
		ConcatenatedCommandLineArguments += FString::Printf(TEXT(" -dc_node=\"%s\""), *Node);
		// Add Multi-User params
		if (!ConcertArguments.IsEmpty())
		{
			ConcatenatedCommandLineArguments += " " + ConcertArguments;
		}

		// Input arguments
		ConcatenatedCommandLineArguments += " " + GetInputArguments();

		// Log file
		const FString LogFileName = (ProjectSettings->LogFileName.IsEmpty() ? Node : ProjectSettings->LogFileName) + ".log";

		// Other cvars
		DPCvars.Add(TEXT("LevelInstance.ForceEditorWorldMode=1"));
		DPCvars.Add(TEXT("p.Chaos.Solver.Deterministic=1"));

		const FString Params =
			FString::Printf(
				TEXT("\"%s\" -game \"%s\" Log=%s %s -ExecCmds=\"%s\" -DPCVars=\"%s\" -LogCmds=\"%s\""),
				*Project, *Map, *LogFileName, *ConcatenatedCommandLineArguments,
				*ConcatenatedConsoleCommands, *FString::Join(DPCvars, TEXT(",")), *ConcatenatedLogCommands
		);
		UE_LOGF(LogDisplayClusterLaunchEditor, Log, "Full Command: %ls %ls", *EditorBinary, *Params);
		void* WritePipe = nullptr;
		ActiveDisplayClusterProcesses.Add(
			FPlatformProcess::CreateProc(
				*EditorBinary, *Params,
				ProjectSettings->bCloseEditorOnLaunch,
				false, false, NULL,
				0, NULL, WritePipe
			)
		);
	}
}

void FDisplayClusterLaunchEditorModule::TerminateActiveDisplayClusterProcesses()
{
	for (FProcHandle& Process : ActiveDisplayClusterProcesses)
	{
		FPlatformProcess::TerminateProc(Process);
		FPlatformProcess::CloseProc(Process);
	}
	RemoveTerminatedNodeProcesses();
}

void FDisplayClusterLaunchEditorModule::OnFEngineLoopInitComplete()
{
	RegisterProjectSettings();
	RegisterToolbarItem();
	RegisterPlacementModeItemsIfTheyExist();
}

void FDisplayClusterLaunchEditorModule::RegisterToolbarItem()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
	
	RemoveToolbarItem();
	FToolMenuSection& Section = Menu->AddSection("DisplayClusterLaunch");
	const FToolMenuEntry DisplayClusterLaunchButton =
		FToolMenuEntry::InitToolBarButton("DisplayClusterLaunchToolbarButton",
			FUIAction(
				FExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::OnClickToolbarButton)
			),
			TAttribute<FText>(),
			TAttribute<FText>::Create(
				TAttribute<FText>::FGetter::CreateRaw(this, &FDisplayClusterLaunchEditorModule::GetToolbarButtonTooltipText)
			),
			TAttribute<FSlateIcon>::Create(
				TAttribute<FSlateIcon>::FGetter::CreateRaw(this, &FDisplayClusterLaunchEditorModule::GetToolbarButtonIcon)
			)
		);
	const FToolMenuEntry DisplayClusterLaunchComboButton = FToolMenuEntry::InitComboButton(
		"DisplayClusterLaunchMenu",
		FUIAction(),
		FOnGetContent::CreateRaw(this, &FDisplayClusterLaunchEditorModule::CreateToolbarMenuEntries),
		LOCTEXT("DisplayClusterLaunchActions", "Launch nDisplay Actions"),
		LOCTEXT("DisplayClusterLaunchActionsTooltip", "Actions related to nDisplay Launch"),
		FSlateIcon(),
		true //bInSimpleComboBox
	);
	Section.AddEntry(DisplayClusterLaunchButton);
	Section.AddEntry(DisplayClusterLaunchComboButton);
}

FText FDisplayClusterLaunchEditorModule::GetToolbarButtonTooltipText()
{
	if (ActiveDisplayClusterProcesses.Num() == 1)
	{
		return LOCTEXT("TerminateActiveProcess","Terminate active nDisplay process");
	}
	else if (ActiveDisplayClusterProcesses.Num() > 1)
	{
		return FText::Format(LOCTEXT("TerminateActiveProcessesFormat","Terminate {0} active nDisplay processes"), ActiveDisplayClusterProcesses.Num());
	}
	if (!SelectedDisplayClusterConfigActor.ResolveObject())
	{
		return LOCTEXT("GenericLaunchDisplayClusterProcessText_NoConfig", "Launch an nDisplay instance using the first Config Actor found in the current level and the first node found in that configuration.\n\nSet specific configurations and nodes using the overflow menu.");
	}
	if (SelectedDisplayClusterConfigActorNodes.Num() == 0)
	{
		return FText::Format(
			LOCTEXT("GenericLaunchDisplayClusterProcessText_NoNodesFormat", "Launch an nDisplay instance using the Config Actor named '{0}' and the first node found in this configuration.\n\nSet specific configurations and nodes using the overflow menu."),
				FText::FromString(SelectedDisplayClusterConfigActor.GetAssetName())
		);
	}
	FString ConfigActorName = SelectedDisplayClusterConfigActor.ResolveObject()->GetName();
	const FString SplitTerm = "_C";
	if (ConfigActorName.Contains(SplitTerm))
	{
		ConfigActorName = ConfigActorName.Left(ConfigActorName.Find(SplitTerm));
	}
	
	return FText::Format(
		LOCTEXT("LaunchDisplayClusterProcessesFormat", "Launch the following nodes:\n\n{0}\n\nFrom this configuration:\n\n{1}"),
			GetSelectedNodesListText(), FText::FromString(ConfigActorName)
		);
}

FSlateIcon FDisplayClusterLaunchEditorModule::GetToolbarButtonIcon()
{
	RemoveTerminatedNodeProcesses();
	return FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),
		ActiveDisplayClusterProcesses.Num() > 0 ? "Icons.DisplayClusterLaunchStop" : "Icons.DisplayClusterLaunchPlay");
}

void FDisplayClusterLaunchEditorModule::OnClickToolbarButton()
{
	if (ActiveDisplayClusterProcesses.Num() == 0)
	{
		TryLaunchDisplayClusterProcess();
	}
	else
	{
		TerminateActiveDisplayClusterProcesses();
	}
}

void FDisplayClusterLaunchEditorModule::RemoveToolbarItem()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
	if (Menu->FindSection("DisplayClusterLaunch"))
	{
		Menu->RemoveSection("DisplayClusterLaunch");
	}
}

void FDisplayClusterLaunchEditorModule::RegisterProjectSettings() const
{
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		// User Project Settings
		const TSharedPtr<ISettingsSection> ProjectSettingsSectionPtr = SettingsModule.RegisterSettings(
			"Project", "Plugins", "nDisplay Launch",
			LOCTEXT("DisplayClusterLaunchSettingsCategoryDisplayName", "nDisplay Launch"),
			LOCTEXT("DisplayClusterLaunchSettingsDescription", "Configure the nDisplay Launch user settings"),
			GetMutableDefault<UDisplayClusterLaunchEditorProjectSettings>());
	}
}

void FDisplayClusterLaunchEditorModule::RegisterPlacementModeItemsIfTheyExist()
{
	if (GEditor)
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("DisplayClusterLaunch"));
		check(Plugin.IsValid());

		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();

		TArray<FAssetData> OutAssets;
		const FString PlaceablePath = Plugin->GetContentDir();
		
		FString PackagePath;
		FString* OutFailureReason = nullptr;
		FPackageName::TryConvertFilenameToLongPackageName(PlaceablePath, PackagePath, OutFailureReason);

		AssetRegistry->ScanPathsSynchronous({PlaceablePath});
		AssetRegistry->GetAssetsByPath(*PackagePath, OutAssets);

		if (OutAssets.Num() > 0)
		{
			if (const FPlacementCategoryInfo* Info = GetDisplayClusterPlacementCategoryInfo())
			{
				for (const FAssetData& Asset : OutAssets)
				{
					TSharedRef<FPlaceableItem> Item =
						MakeShared<FPlaceableItem>(
							*UActorFactoryBlueprint::StaticClass(),
							Asset
						);

					Item->DisplayName =
						FText::FromString(
							Item->DisplayName.ToString().Replace(TEXT("_"), TEXT(" "))
						);
					
					IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, Item);
				}
			}
		}
	}
}

const FPlacementCategoryInfo* FDisplayClusterLaunchEditorModule::GetDisplayClusterPlacementCategoryInfo()
{
	if (GEditor)
	{
		const IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
		const FName PlacementModeCategoryHandle = TEXT("nDisplay");

		if (const FPlacementCategoryInfo* RegisteredInfo = PlacementModeModule.GetRegisteredPlacementCategory(PlacementModeCategoryHandle))
		{
			return RegisteredInfo;
		}
		else
		{
			FPlacementCategoryInfo Info(
				LOCTEXT("DisplayClusterCategoryName", "nDisplay"),
				FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(), "Icons.DisplayCluster"),
				PlacementModeCategoryHandle,
				TEXT("nDisplay"),
				25
			);

			IPlacementModeModule::Get().RegisterPlacementCategory(Info);

			// This will return nullptr if the Register above failed so we don't need to explicitly check
			// RegisterPlacementCategory's return value.
			return PlacementModeModule.GetRegisteredPlacementCategory(PlacementModeCategoryHandle);
		}
	}

	return nullptr;
}

FText FDisplayClusterLaunchEditorModule::GetSelectedNodesListText() const
{
	if (SelectedDisplayClusterConfigActorNodes.Num() > 0)
	{
		FString JoinedNodes = FString::Join(SelectedDisplayClusterConfigActorNodes, TEXT("\n"));
		int32 IndexOfBreakOrLengthIfNoBreakFound = JoinedNodes.Find(TEXT("\n"));
		if (IndexOfBreakOrLengthIfNoBreakFound == INDEX_NONE) { IndexOfBreakOrLengthIfNoBreakFound = JoinedNodes.Len(); }
		JoinedNodes.InsertAt(IndexOfBreakOrLengthIfNoBreakFound, " ({0})");
	
		return FText::Format(FText::FromString(JoinedNodes), LOCTEXT("PrimaryNode", "Primary"));
	}
	return FText::GetEmpty();
}

void FDisplayClusterLaunchEditorModule::GetProjectSettingsArguments(
	const UDisplayClusterLaunchEditorProjectSettings* ProjectSettings, FString& ConcatenatedCommandLineArguments,
	FString& ConcatenatedConsoleCommands, TSet<FString>& DPCvars, FString& ConcatenatedLogCommands)
{
	{
		for (const FString& CommandLineArgument : ProjectSettings->CommandLineArguments)
		{
			if (!CommandLineArgument.IsEmpty())
			{
				ConcatenatedCommandLineArguments += FString::Printf(TEXT(" -%s "), *CommandLineArgument);
			}
		}
		// Remove whitespace
		ConcatenatedCommandLineArguments.TrimStartAndEndInline();
	}

	// Add console commands
	TArray<FString> CollectedCommands;

	auto ConsoleVariablesAssetLambda = [](FSoftObjectPath InAssetSoftPath, TArray<FString> OutCollectedCommands)
	{
		if (InAssetSoftPath.IsValid())
		{
			if (TObjectPtr<UConsoleVariablesAsset> CvarAsset = Cast<UConsoleVariablesAsset>(InAssetSoftPath.TryLoad()))
			{
				OutCollectedCommands.Append(CvarAsset->GetSavedCommandsAsStringArray());
			}
			else
			{
				UE_LOGF(LogDisplayClusterLaunchEditor, Warning, "%s: The specified Console Variables Asset was not valid: %ls.", __FUNCTION__, *InAssetSoftPath.GetAssetName());
			}
		}
	};

	ConsoleVariablesAssetLambda(ProjectSettings->ConsoleVariablesPreset, CollectedCommands);
	ConsoleVariablesAssetLambda(SelectedAdditionalConsoleVariablesAsset, CollectedCommands);

	// Additional Console Commands
	CollectedCommands.Append(ProjectSettings->AdditionalConsoleCommands.Array());
	
	ConcatenatedConsoleCommands += FString::Join(CollectedCommands, TEXT(","));

	// Additional Console Variables
	DPCvars.Append(ProjectSettings->AdditionalConsoleVariables);

	// Logging
	{
		for (const FDisplayClusterLaunchLoggingConstruct& LoggingConstruct : ProjectSettings->Logging)
		{
			if (!LoggingConstruct.Category.IsNone())
			{
				ConcatenatedLogCommands +=
					FString::Printf(TEXT("%s %s, "),
						*LoggingConstruct.Category.ToString(),
						*EnumToString("EDisplayClusterLaunchLogVerbosity", (int32)LoggingConstruct.VerbosityLevel.GetValue()));
			}
		}
		
		// Remove whitespace
		ConcatenatedLogCommands.TrimStartAndEndInline();
		// Remove last comma
		ConcatenatedLogCommands = ConcatenatedLogCommands.LeftChop(1);
	}
}


TArray<TWeakObjectPtr<ADisplayClusterRootActor>> FDisplayClusterLaunchEditorModule::GetAllDisplayClusterConfigsInWorld()
{
	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> CachedDisplayClusterActors;
	for (TActorIterator<ADisplayClusterRootActor> Itr(GetCurrentWorld()); Itr; ++Itr)
	{
		if (ADisplayClusterRootActor* RootActor = *Itr)
		{
			CachedDisplayClusterActors.Add(RootActor);
		}
	}
	bAreConfigsFoundInWorld = CachedDisplayClusterActors.Num() > 0;
	CachedDisplayClusterActors.Sort([](const TWeakObjectPtr<ADisplayClusterRootActor>& A, const TWeakObjectPtr<ADisplayClusterRootActor>& B)
	{
		return A->GetActorLabel() < B->GetActorLabel();
	});
	return CachedDisplayClusterActors;
}

bool FDisplayClusterLaunchEditorModule::DoesCurrentWorldHaveDisplayClusterConfig()
{
	return bAreConfigsFoundInWorld;
}

void FDisplayClusterLaunchEditorModule::ApplyDisplayClusterConfigOverrides(UDisplayClusterConfigurationData* ConfigDataCopy)
{
	if (!ConfigDataCopy->Scene)
	{
		ConfigDataCopy->Scene = NewObject<UDisplayClusterConfigurationScene>(ConfigDataCopy, NAME_None,
		RF_ArchetypeObject | RF_Public );
	}
	// A Primary Node should always be automatically selected, but this code preempts a crash. Normally we use the PN specified in the UI.
	// If one is not specified in the UI, we check to see if the primary node specified
	// in the original config is in our node array selection from the UI.
	// If it isn't, in the loop below we'll use the first active node.
	bool bIsConfigPrimaryNodeInActiveNodes = false;
	const bool bIsPrimaryNodeUnset = SelectedDisplayClusterConfigActorPrimaryNode.IsEmpty();
	if (bIsPrimaryNodeUnset)
	{
		bIsConfigPrimaryNodeInActiveNodes =
			SelectedDisplayClusterConfigActorNodes.Contains(ConfigDataCopy->Cluster->PrimaryNode.Id);
	}
	else
	{
		ConfigDataCopy->Cluster->PrimaryNode.Id = SelectedDisplayClusterConfigActorPrimaryNode;
	}

	using NodeMapType = decltype(UDisplayClusterConfigurationCluster::Nodes);
	NodeMapType ActiveNodes;
	NodeMapType NodesInConfig = ConfigDataCopy->Cluster->Nodes;
	for (int32 NodeIndex = 0; NodeIndex < SelectedDisplayClusterConfigActorNodes.Num(); NodeIndex++)
	{
		FString NodeId = SelectedDisplayClusterConfigActorNodes[NodeIndex];
		if (TObjectPtr<UDisplayClusterConfigurationClusterNode>* Node = NodesInConfig.Find(NodeId))
		{
			ActiveNodes.Add(NodeId, *Node);
			(*Node)->Host = "127.0.0.1";
			// If we haven't specified a primary node and the config's primary node is not in our selection, use the first active node.
			if (bIsPrimaryNodeUnset && !bIsConfigPrimaryNodeInActiveNodes && ActiveNodes.Num() == 1)
			{
				ConfigDataCopy->Cluster->PrimaryNode.Id = NodeId;
			}
		}
	}
	ConfigDataCopy->Cluster->Nodes = ActiveNodes;
}

void FDisplayClusterLaunchEditorModule::RefreshSelectedDisplayActor()
{
	if (!SelectedDisplayClusterConfigActor.IsValid() || !Cast<ADisplayClusterRootActor>(SelectedDisplayClusterConfigActor.ResolveObject()))
	{
		for (const TWeakObjectPtr<ADisplayClusterRootActor>& Config : GetAllDisplayClusterConfigsInWorld())
		{
			if (ADisplayClusterRootActor* ConfigPtr = Config.Get())
			{
				TArray<FString> NodeNames;
				ConfigPtr->GetConfigData()->Cluster->Nodes.GenerateKeyArray(NodeNames);
				if (NodeNames.Num() > 0)
				{
					SetSelectedDisplayClusterConfigActor(ConfigPtr);
					break;
				}
			}
		}
	}
}

void FDisplayClusterLaunchEditorModule::SetSelectedDisplayClusterConfigActor(ADisplayClusterRootActor* SelectedActor)
{
	if (SelectedActor)
	{
		const FSoftObjectPath AsSoftObjectPath(SelectedActor);
		if (AsSoftObjectPath != SelectedDisplayClusterConfigActor)
		{
			SelectedDisplayClusterConfigActor = AsSoftObjectPath;
			SelectedDisplayClusterConfigActorNodes.Empty();
			SelectFirstNode(SelectedActor);
		}
	}
}

void FDisplayClusterLaunchEditorModule::ToggleDisplayClusterConfigActorNodeSelected(FString InNodeName)
{
	if (IsDisplayClusterConfigActorNodeSelected(InNodeName))
	{
		SelectedDisplayClusterConfigActorNodes.Remove(InNodeName);
	}
	else
	{
		SelectedDisplayClusterConfigActorNodes.Add(InNodeName);
	}
	
	// Clear SelectedDisplayClusterConfigActorPrimaryNode if no nodes are selected
	if (SelectedDisplayClusterConfigActorNodes.Num() == 0)
	{
		SelectedDisplayClusterConfigActorPrimaryNode = "";
	}
	
	// If a single node is selected, SelectedDisplayClusterConfigActorPrimaryNode must be this node
	if (SelectedDisplayClusterConfigActorNodes.Num() == 1)
	{
		SelectedDisplayClusterConfigActorPrimaryNode = SelectedDisplayClusterConfigActorNodes[0];
	}
}

bool FDisplayClusterLaunchEditorModule::IsDisplayClusterConfigActorNodeSelected(FString InNodeName)
{
	return SelectedDisplayClusterConfigActorNodes.Contains(InNodeName);
}

void FDisplayClusterLaunchEditorModule::SetSelectedConsoleVariablesAsset(const FAssetData InConsoleVariablesAsset)
{
	if (SelectedAdditionalConsoleVariablesAsset.GetAssetName().Equals(InConsoleVariablesAsset.AssetName.ToString()))
	{
		SelectedAdditionalConsoleVariablesAsset.Reset();
	}
	else
	{
		SelectedAdditionalConsoleVariablesAsset = InConsoleVariablesAsset.ToSoftObjectPath();
	}
}

void FDisplayClusterLaunchEditorModule::SelectFirstNode(ADisplayClusterRootActor* InConfig)
{
	TArray<FString> NodeNames;
	InConfig->GetConfigData()->Cluster->Nodes.GenerateKeyArray(NodeNames);
	if (NodeNames.Num() == 0)
	{
		UE_LOGF(LogDisplayClusterLaunchEditor, Error, "%s: Unable to launch nDisplay because there are no nDisplay nodes in the selected nDisplay Config named '%ls'.", __FUNCTION__, *InConfig->GetActorNameOrLabel());
		return;
	}
		
	SelectedDisplayClusterConfigActorNodes.SetNum(
		Algo::StableRemoveIf(
			SelectedDisplayClusterConfigActorNodes,
			[NodeNames](const FString& SelectedNode)
			{
				return !NodeNames.Contains(SelectedNode);
			}
		)
	);
	
	if (SelectedDisplayClusterConfigActorNodes.Num() == 0)
	{
		const FString& NodeName = NodeNames[0];
		UE_LOGF(LogDisplayClusterLaunchEditor, Verbose, "%s: Selected nDisplay nodes were not found on the selected DisplayClusterRootActor. We will select the first valid node.", __FUNCTION__);
		SelectedDisplayClusterConfigActorNodes.Add(NodeName);
		UE_LOGF(LogDisplayClusterLaunchEditor, Verbose, "%s: Adding first valid node named '%ls' to selected nodes.", __FUNCTION__, *NodeName);
	}

	// Set the primary node to be the first in the list
	SelectedDisplayClusterConfigActorPrimaryNode = SelectedDisplayClusterConfigActorNodes[0];
}

TSharedRef<SWidget>  FDisplayClusterLaunchEditorModule::CreateToolbarMenuEntries()
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	
	FMenuBuilder MenuBuilder(false, nullptr);
	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> DisplayClusterConfigs = GetAllDisplayClusterConfigsInWorld();
	MenuBuilder.BeginSection("DisplayClusterLaunch", LOCTEXT("DisplayClusterLauncher", "Launch nDisplay"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisplayClusterLaunchLastNode", "Launch Last Node Configuration"),
			LOCTEXT("DisplayClusterLaunchLastNodeTooltip", "Launch the last node configuration."),
			FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(), "Icons.DisplayClusterLaunchPlay"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::TryLaunchDisplayClusterProcess),
				FCanExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::DoesCurrentWorldHaveDisplayClusterConfig)
			),
			NAME_None
		);
	}
	MenuBuilder.EndSection();
	AddDisplayClusterLaunchConfigurations(AssetRegistry, MenuBuilder, DisplayClusterConfigs);
	AddDisplayClusterLaunchNodes(AssetRegistry, MenuBuilder);
	AddConsoleVariablesEditorAssetsToToolbarMenu(AssetRegistry, MenuBuilder);
	AddOptionsToToolbarMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

void FDisplayClusterLaunchEditorModule::AddDisplayClusterLaunchConfigurations(IAssetRegistry* AssetRegistry,
	FMenuBuilder& MenuBuilder, TArray<TWeakObjectPtr<ADisplayClusterRootActor>>& DisplayClusterConfigs)
{
	MenuBuilder.BeginSection("DisplayClusterLaunchConfigurations", LOCTEXT("DisplayClusterLaunchConfigurations", "Configuration"));
	{
		if (DisplayClusterConfigs.Num())
		{
			// If one is not set, select the first one found
			bool bIsConfigActorValid = false;
			if (ADisplayClusterRootActor* SelectedActor = Cast<ADisplayClusterRootActor>(SelectedDisplayClusterConfigActor.ResolveObject()))
			{
				bIsConfigActorValid = DisplayClusterConfigs.ContainsByPredicate([SelectedActor](TWeakObjectPtr<ADisplayClusterRootActor>& Comparator)
				{
					return Comparator.IsValid() && SelectedActor == Comparator.Get();
				});
			}
			if (!bIsConfigActorValid)
			{
				SetSelectedDisplayClusterConfigActor(DisplayClusterConfigs[0].Get());
			}
			
			for (const TWeakObjectPtr<ADisplayClusterRootActor>& Node : DisplayClusterConfigs)
			{
				if (!Node.IsValid())
				{
					continue;
				}
				
				const FText NodeName = FText::FromString(Node->GetActorLabel());
				const FText DisplayClusterConfigTooltip =
					FText::Format(LOCTEXT("SelectDisplayClusterConfigFormat","Select configuration '{0}'"), NodeName);
			
				MenuBuilder.AddMenuEntry(
					NodeName,
					DisplayClusterConfigTooltip,
					FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),"Icons.DisplayCluster"),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::SetSelectedDisplayClusterConfigActor, Node.Get()),
						FCanExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::DoesCurrentWorldHaveDisplayClusterConfig),
						FIsActionChecked::CreateLambda([this, Node]() { return SelectedDisplayClusterConfigActor == FSoftObjectPath(Node.Get()); })
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
		else
		{
			SelectedDisplayClusterConfigActor.Reset();
			
			MenuBuilder.AddWidget(SNullWidget::NullWidget,
				LOCTEXT("NoDisplayClusterConfigAssetInLevelText", "Add an nDisplay configuration asset to\nthe current level to launch nDisplay."));
		}
	}
	MenuBuilder.EndSection();
}

void FDisplayClusterLaunchEditorModule::AddDisplayClusterLaunchNodes(IAssetRegistry* AssetRegistry, FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("DisplayClusterLaunchNodes", LOCTEXT("DisplayClusterLaunchNodes", "Nodes"));
	{
		// Submenu for node selection. Using a WrapperSubMenu to avoid the menu automatically closing when selecting nodes
		// AddWrapperSubMenu does not allow for TAttribute<FText> Labels, it just copies the FText input so we need this entry to display live data
		MenuBuilder.AddMenuEntry(
			TAttribute<FText>::CreateLambda(
				[this] ()
				{
					const int32 NodeCount = SelectedDisplayClusterConfigActorNodes.Num();
										
					if (NodeCount > 0)
					{
						if (NodeCount == 1)
						{
							return FText::Format(
								LOCTEXT("SelectedSingleNodeFormat", "'{0}' Selected"),
								FText::FromString(SelectedDisplayClusterConfigActorNodes[0]));
						}
						else
						{
							return FText::Format(
								LOCTEXT("SelectedMultipleNodesFormat", "Selected {0} Nodes"), FText::AsNumber(NodeCount));
						}
					}
					else
					{
						return LOCTEXT("NoDisplayClusterLaunchNodesSelected", "Please select nDisplay nodes to launch.");
					}
				}
			),
			TAttribute<FText>::CreateRaw(
					this, &FDisplayClusterLaunchEditorModule::GetSelectedNodesListText
			),
			FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),"Icons.DisplayClusterNode"),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([]() { return false; })
			),
			NAME_None,
			EUserInterfaceActionType::None
		);
		
		MenuBuilder.AddWrapperSubMenu(
			LOCTEXT("SelectDisplayClusterNodes","Select nDisplay Nodes"),
			LOCTEXT("SelectDisplayClusterNodesTooltip","Select nDisplay Nodes.\nThe first node selected will be designated as the primary node when launched unless otherwise specified."),
			FOnGetContent::CreateLambda([this]()
			{
				FMenuBuilder NewMenuBuilder(false, nullptr);
				NewMenuBuilder.AddSubMenu(
					TAttribute<FText>::CreateLambda([this]()
					{
						return
							FText::Format(
								LOCTEXT("SelectPrimaryNodeFormat","Select Primary Node ({0})"),
								SelectedDisplayClusterConfigActorPrimaryNode.IsEmpty() ? LOCTEXT("None", "None") :
								FText::FromString(SelectedDisplayClusterConfigActorPrimaryNode));
					}),
					LOCTEXT("SelectPrimaryNode", "Select the Primary Node"),
					FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
					{
						const FText NodeTooltip =
							LOCTEXT("MakePrimaryNodeTooltip", "Make this node the new Primary Node. Does not affect the original configuration.");
						
						for (const FString& SelectedNode : SelectedDisplayClusterConfigActorNodes)
						{
							InMenuBuilder.AddMenuEntry(
								FText::FromString(SelectedNode),
								NodeTooltip,
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([this, SelectedNode]()
									{
										SelectedDisplayClusterConfigActorPrimaryNode = SelectedNode;
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([this, SelectedNode]()
									{
										return SelectedDisplayClusterConfigActorPrimaryNode == SelectedNode;
									})
								),
								NAME_None,
								EUserInterfaceActionType::RadioButton
							);
						}
					}),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction::CreateLambda([this]()
						{
							return SelectedDisplayClusterConfigActorNodes.Num() > 0;
						})
					),
					NAME_None,
					EUserInterfaceActionType::None
				);
				NewMenuBuilder.AddSeparator();
				if (ADisplayClusterRootActor* SelectedActor = Cast<ADisplayClusterRootActor>(SelectedDisplayClusterConfigActor.ResolveObject()))
				{
					TArray<FString> NodeNames;
				   SelectedActor->GetConfigData()->Cluster->Nodes.GenerateKeyArray(NodeNames);
				
				   for (const FString& NodeName : NodeNames)
				   {
					   const FText DisplayClusterNodeName = FText::FromString(NodeName);
					
					   const FText DisplayClusterNodeTooltip =
						   FText::Format(LOCTEXT("SelectDisplayClusterNodeFormat","Select node '{0}'"), DisplayClusterNodeName);
					
					   NewMenuBuilder.AddMenuEntry(
						   DisplayClusterNodeName,
						   DisplayClusterNodeTooltip,
						   FSlateIcon(),
						   FUIAction(
							   FExecuteAction::CreateRaw(
								   this, &FDisplayClusterLaunchEditorModule::ToggleDisplayClusterConfigActorNodeSelected, NodeName),
							   FCanExecuteAction(),
							   FIsActionChecked::CreateRaw(
								   this, &FDisplayClusterLaunchEditorModule::IsDisplayClusterConfigActorNodeSelected, NodeName)
						   ),
						   NAME_None,
						   EUserInterfaceActionType::Check
					   );
				   }
				}
				return NewMenuBuilder.MakeWidget();
			}),
			FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),"Icons.DisplayClusterNode")
		);
		MenuBuilder.EndSection();
	}
}

void FDisplayClusterLaunchEditorModule::AddConsoleVariablesEditorAssetsToToolbarMenu(
	IAssetRegistry* AssetRegistry, FMenuBuilder& MenuBuilder)
{
	TArray<FAssetData> FoundConsoleVariablesAssets;
	AssetRegistry->GetAssetsByClass(FTopLevelAssetPath(TEXT("/Script/ConsoleVariablesEditorRuntime"), TEXT("ConsoleVariablesAsset")), FoundConsoleVariablesAssets, true);
	
	if (FoundConsoleVariablesAssets.Num())
	{
		MenuBuilder.BeginSection("DisplayClusterLaunchCvars", LOCTEXT("DisplayClusterLaunch_AdditionalConsoleVariablesAsset", "Additional Console Variables Asset"));
		{
			const FText ConsoleVariablesAssetTooltip = LOCTEXT("SelectConsoleVariablesAssetTooltip","Select Additional Console Variables Asset. This asset will be in addition to and its commands will be executed after the one defined in project settings.");
			
			MenuBuilder.AddSubMenu(
				TAttribute<FText>::Create(
					[this]()
					{
						FText OutText = SelectedAdditionalConsoleVariablesAsset.IsValid() ?
							FText::FromString(SelectedAdditionalConsoleVariablesAsset.GetAssetName()) :
							LOCTEXT("NoneSelected", "None Selected");
							
						return OutText;
					}),
				ConsoleVariablesAssetTooltip,
				FNewMenuDelegate::CreateLambda([this, FoundConsoleVariablesAssets](FMenuBuilder& NewMenuBuilder)
				{
					for (const FAssetData& Asset : FoundConsoleVariablesAssets)
					{
						const FText ConsoleVariablesAssetName = FText::FromName(Asset.AssetName);
						const FText ConsoleVariablesAssetTooltip =
							FText::Format(LOCTEXT("SelectConsoleVariablesAssetFormatLong","Select Console Variables Asset '{0}'"), ConsoleVariablesAssetName);
						NewMenuBuilder.AddMenuEntry(
							ConsoleVariablesAssetName,
							ConsoleVariablesAssetTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::SetSelectedConsoleVariablesAsset, Asset),
								FCanExecuteAction(),
								FGetActionCheckState::CreateLambda([this, Asset](){ return SelectedAdditionalConsoleVariablesAsset.GetAssetName().Equals(Asset.AssetName.ToString()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							),
							NAME_None,
							EUserInterfaceActionType::RadioButton
						);
					}
				}),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateRaw(this, &FDisplayClusterLaunchEditorModule::DoesCurrentWorldHaveDisplayClusterConfig)),
				NAME_None,
				EUserInterfaceActionType::None,
				false,
				FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),"Icons.ConsoleVariablesEditor")
			);
		}
		MenuBuilder.EndSection();
	}
}

void FDisplayClusterLaunchEditorModule::AddOptionsToToolbarMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("DisplayClusterLaunchOptions", LOCTEXT("DisplayClusterLaunchOptions", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ConnectToMultiUserLabel", "Connect to Multi-User"),
			LOCTEXT("ConnectToMultiUserTooltip", "Connect to Multi-User"),
			FSlateIcon(FDisplayClusterLaunchEditorStyle::Get().GetStyleSetName(),"Icons.MultiUser"),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					UDisplayClusterLaunchEditorProjectSettings* Settings = GetMutableDefault<UDisplayClusterLaunchEditorProjectSettings>();
					Settings->bConnectToMultiUser = !GetConnectToMultiUser();
					Settings->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FDisplayClusterLaunchEditorModule::GetConnectToMultiUser)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("EnableUnrealInsightsLabel", "Enable Unreal Insights"),
			LOCTEXT("EnableUnrealInsightsTooltip", "Enable Unreal Insights"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(),"UnrealInsights.MenuIcon"),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					UDisplayClusterLaunchEditorProjectSettings* Settings = GetMutableDefault<UDisplayClusterLaunchEditorProjectSettings>();
					Settings->bEnableUnrealInsights = !Settings->bEnableUnrealInsights;
					Settings->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return GetDefault<UDisplayClusterLaunchEditorProjectSettings>()->bEnableUnrealInsights; })
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CloseEditorOnLaunchLabel", "Close Editor on Launch"),
			LOCTEXT("CloseEditorOnLaunchTooltip", "Close Editor on Launch"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(),"Icons.X"),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					UDisplayClusterLaunchEditorProjectSettings* Settings = GetMutableDefault<UDisplayClusterLaunchEditorProjectSettings>();
					Settings->bCloseEditorOnLaunch = !Settings->bCloseEditorOnLaunch;
					Settings->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return GetDefault<UDisplayClusterLaunchEditorProjectSettings>()->bCloseEditorOnLaunch; })
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AdvancedSettingsLabel", "Advanced Settings..."),
			LOCTEXT("AdvancedSettingsTooltip", "Open the nDisplay Launch Project Settings"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(),"Icons.Settings"),
			FUIAction(
				FExecuteAction::CreateStatic(&FDisplayClusterLaunchEditorModule::OpenProjectSettings)
			),
			NAME_None
		);
	}
	MenuBuilder.EndSection();
}

bool FDisplayClusterLaunchEditorModule::GetConnectToMultiUser() const
{
	return GetDefault<UDisplayClusterLaunchEditorProjectSettings>()->bConnectToMultiUser;
}

const FString& FDisplayClusterLaunchEditorModule::GetConcertServerName()
{
	// If the Cached name is changed after this point it will need to be cleared in order to run this code again 
	if (ServerTrackingData.GeneratedMultiUserServerName.IsEmpty())
	{
		ServerTrackingData.GeneratedMultiUserServerName = AppendRandomNumbersToString("nDisplayLaunchServer");
	}
	return ServerTrackingData.GeneratedMultiUserServerName;
}

const FString& FDisplayClusterLaunchEditorModule::GetConcertSessionName()
{
	// If the Cached name is changed after this point it will need to be cleared in order to run this code again 
	if (CachedConcertSessionName.IsEmpty())
	{
		const UDisplayClusterLaunchEditorProjectSettings* Settings = GetDefault<UDisplayClusterLaunchEditorProjectSettings>();
		if (!Settings->ExplicitSessionName.IsEmpty())
		{
			CachedConcertSessionName = Settings->ExplicitSessionName;
		}
		else
		{
			CachedConcertSessionName = AppendRandomNumbersToString("nDisplayLaunchSession");
		}
	}
	return CachedConcertSessionName;
}

void FDisplayClusterLaunchEditorModule::RemoveTerminatedNodeProcesses()
{
	ActiveDisplayClusterProcesses.SetNum(
		Algo::StableRemoveIf(
			ActiveDisplayClusterProcesses,
			[](FProcHandle& Handle)
			{
				return !FPlatformProcess::IsProcRunning(Handle);
			}
		)
	);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDisplayClusterLaunchEditorModule, DisplayClusterLaunchEditor)
