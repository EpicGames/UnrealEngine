// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLaunchEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

TSharedPtr<FSlateStyleSet> FDisplayClusterLaunchEditorStyle::StyleInstance = nullptr;

void FDisplayClusterLaunchEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FDisplayClusterLaunchEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

void FDisplayClusterLaunchEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FDisplayClusterLaunchEditorStyle::Get()
{
	return *StyleInstance;
}

const FName& FDisplayClusterLaunchEditorStyle::GetStyleSetName() const
{
	static FName DisplayClusterLaunchStyleSetName(TEXT("DisplayClusterLaunchEditor"));
	return DisplayClusterLaunchStyleSetName;
}

const FSlateBrush* FDisplayClusterLaunchEditorStyle::GetBrush(const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	return StyleInstance->GetBrush(PropertyName, Specifier);
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir(RelativePath, TEXT(".svg") ), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG( PluginName, RelativePath, ... ) FSlateVectorImageBrush( FDisplayClusterLaunchEditorStyle::GetExternalPluginContent(PluginName, RelativePath, ".svg"), __VA_ARGS__)

const FVector2D Icon64x64(64.f, 64.f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);
const FVector2D Icon8x8(8.f, 8.f);

FString FDisplayClusterLaunchEditorStyle::GetExternalPluginContent(const FString& PluginName, const FString& RelativePath, const ANSICHAR* Extension)
{
	FString ContentDir = IPluginManager::Get().FindPlugin(PluginName)->GetBaseDir() / RelativePath + Extension;
	return ContentDir;
}

TSharedRef< FSlateStyleSet > FDisplayClusterLaunchEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("DisplayClusterLaunchEditor");
	
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("DisplayClusterLaunch")))
	{
		Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}

	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Icons
	Style->Set("Icons.DisplayClusterLaunchPlay", new IMAGE_BRUSH_SVG("Icons/nDisplayQuickLaunchPlay_20", Icon16x16));
	Style->Set("Icons.DisplayClusterLaunchStop", new IMAGE_BRUSH_SVG("Icons/nDisplayQuickLaunchX_20", Icon16x16));

	// External plugin icons
	
	Style->Set("Icons.MultiUser", new IMAGE_PLUGIN_BRUSH_SVG("ConcertSharedSlate","Content/Icons/icon_MultiUser", Icon16x16));
	Style->Set("Icons.DisplayCluster", new IMAGE_PLUGIN_BRUSH_SVG("nDisplay","Content/Icons/RootActor/nDisplay", Icon16x16));
	Style->Set("Icons.DisplayClusterNode", new IMAGE_PLUGIN_BRUSH_SVG("nDisplay","Content/Icons/Cluster/ClusterNode", Icon16x16));
	
	
	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
