// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "IConcertModule.h"

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FPlacementCategoryInfo;
class UDisplayClusterLaunchEditorProjectSettings;
class ADisplayClusterRootActor;
class FMenuBuilder;
class FName;
class FString;
class FText;
class FUICommandList;
class IAssetRegistry;
class ISettingsSection;
class SWidget;

struct FAssetData;
struct FProcHandle;
struct FSoftObjectPath;
struct FSlateIcon;

class FDisplayClusterLaunchEditorModule : public IModuleInterface
{
public:

	enum class EConcertServerRequestStatus
	{
		None,
		ShutdownRequested,
		LaunchRequested,
		ReuseExisting
	};

	struct FDisplayClusterLaunchMultiUserServerTrackingData
	{
		FString GeneratedMultiUserServerName;
		FConcertServerInfo MultiUserServerInfo;
		FProcHandle MultiUserServerHandle;
	};
	
	static FDisplayClusterLaunchEditorModule& Get();

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface
	
	static void OpenProjectSettings();

	/**
	 * Performs a sanity check to determine if nDisplay can be launched
	 * and if so, gets Concert parameters async if Concert is enabled for the launch.
	 * Ultimately, LaunchDisplayClusterProcess will be called when Concert parameters are skipped or returned.
	 */
	void TryLaunchDisplayClusterProcess();
	void TerminateActiveDisplayClusterProcesses();

private:
	void EnsureMultiUserDiscovery();
	void StopMultiUserDiscovery();

	void OnFEngineLoopInitComplete();

	void LaunchConcertServer();

	void FindOrLaunchConcertServer();
	void OnServersAssumedReady();
	void FindAppropriateServer();
	void ConnectToSession();
	
	void LaunchDisplayClusterProcess();

	void RegisterToolbarItem();
	FText GetToolbarButtonTooltipText();
	FSlateIcon GetToolbarButtonIcon();
	void OnClickToolbarButton();
	void RemoveToolbarItem();
	
	void RegisterProjectSettings() const;

	static void RegisterPlacementModeItemsIfTheyExist();
	static const FPlacementCategoryInfo* GetDisplayClusterPlacementCategoryInfo();

	/** Returns a list of selected nodes as FText separated by new lines with the Primary Node marked. */
	FText GetSelectedNodesListText() const;

	void GetProjectSettingsArguments(
		const UDisplayClusterLaunchEditorProjectSettings* ProjectSettings, FString& ConcatenatedCommandLineArguments, 
		FString& ConcatenatedConsoleCommands, TSet<FString>& DPCvars, FString& ConcatenatedLogCommands);

	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> GetAllDisplayClusterConfigsInWorld();
	bool DoesCurrentWorldHaveDisplayClusterConfig();
	void ApplyDisplayClusterConfigOverrides(class UDisplayClusterConfigurationData* ConfigDataCopy);

	/** Updates the selected nDisplay actor if invalid or unresolvable */
	void RefreshSelectedDisplayActor();

	void SetSelectedDisplayClusterConfigActor(ADisplayClusterRootActor* SelectedActor);
	void ToggleDisplayClusterConfigActorNodeSelected(FString InNodeName);
	bool IsDisplayClusterConfigActorNodeSelected(FString InNodeName);
	void SetSelectedConsoleVariablesAsset(const FAssetData InConsoleVariablesAsset);

	void SelectFirstNode(ADisplayClusterRootActor* InConfig);
	
	TSharedRef<SWidget> CreateToolbarMenuEntries();
	void AddDisplayClusterLaunchConfigurations(
		IAssetRegistry* AssetRegistry, FMenuBuilder& MenuBuilder, TArray<TWeakObjectPtr<ADisplayClusterRootActor>>& DisplayClusterConfigs);
	void AddDisplayClusterLaunchNodes(class IAssetRegistry* AssetRegistry, FMenuBuilder& MenuBuilder);
	void AddConsoleVariablesEditorAssetsToToolbarMenu(IAssetRegistry* AssetRegistry, FMenuBuilder& MenuBuilder);
	void AddOptionsToToolbarMenu(FMenuBuilder& MenuBuilder);

	bool GetConnectToMultiUser() const;
	const FString& GetConcertServerName();
	const FString& GetConcertSessionName(); 

	void RemoveTerminatedNodeProcesses();

	bool bAreConfigsFoundInWorld = false;

	FSoftObjectPath SelectedDisplayClusterConfigActor;
	TArray<FString> SelectedDisplayClusterConfigActorNodes;
	FString SelectedDisplayClusterConfigActorPrimaryNode;
	
	FSoftObjectPath SelectedAdditionalConsoleVariablesAsset;

	FDisplayClusterLaunchMultiUserServerTrackingData ServerTrackingData;
	FString CachedConcertSessionName;
	
	TArray<FProcHandle> ActiveDisplayClusterProcesses;

	EConcertServerRequestStatus ConcertServerRequestStatus = EConcertServerRequestStatus::None;

	bool bDidStartDiscovery = false;
};
