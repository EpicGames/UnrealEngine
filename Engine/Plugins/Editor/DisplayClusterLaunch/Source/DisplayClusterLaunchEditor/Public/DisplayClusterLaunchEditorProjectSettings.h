// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterLaunchEditorProjectSettings.generated.h"

UENUM()
enum EDisplayClusterLaunchLogVerbosity : int
{
	/** Always prints a fatal error to console (and log file) and crashes (even if logging is disabled) */
	Fatal,

	/** 
	 * Prints an error to console (and log file). 
	 * Commandlets and the editor collect and report errors. Error messages result in commandlet failure.
	 */
	Error,

	/** 
	 * Prints a warning to console (and log file).
	 * Commandlets and the editor collect and report warnings. Warnings can be treated as an error.
	 */
	Warning,

	/** Prints a message to console (and log file) */
	Display,

	/** Prints a message to a log file (does not print to console) */
	Log,

	/** 
	 * Prints a verbose message to a log file (if Verbose logging is enabled for the given category, 
	 * usually used for detailed logging) 
	 */
	Verbose,

	/** 
	 * Prints a verbose message to a log file (if VeryVerbose logging is enabled, 
	 * usually used for detailed logging that would otherwise spam output) 
	 */
	VeryVerbose
};

USTRUCT()
struct FDisplayClusterLaunchLoggingConstruct
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, Category="nDisplay Launch Settings")
	FName Category;

	UPROPERTY(config, EditAnywhere, Category="nDisplay Launch Settings")
	TEnumAsByte<EDisplayClusterLaunchLogVerbosity> VerbosityLevel = EDisplayClusterLaunchLogVerbosity::VeryVerbose;
};

UCLASS(config = DisplayClusterLaunch, defaultconfig)
class DISPLAYCLUSTERLAUNCHEDITOR_API UDisplayClusterLaunchEditorProjectSettings : public UObject
{
	GENERATED_BODY()
	
public:

	/**
	 * If true, the editor will be closed on session launch to optimize session performance.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings")
	bool bCloseEditorOnLaunch = false;

	/**
	 * If true, this command will attempt to connect to an existing
	 * session or create a new Multi-User session with the
	 * options specified in the Multi-User Editing project settings.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Multi-User")
	bool bConnectToMultiUser = false;

	/**
	 * A specific session name to use when connecting to multi-user if bAutoGenerateSessionName is false.
	 * If left empty, a name will be generated.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Multi-User", meta = (EditCondition="bConnectToMultiUser"))
	FString ExplicitSessionName;

	/**
	 * Whether or not to enable Unreal Insights for this session
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Unreal Insights")
	bool bEnableUnrealInsights = false;

	/**
	 * Enable support for Stat Named Events in Unreal Insights
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Unreal Insights", meta = (EditCondition="bEnableUnrealInsights"))
	bool bEnableStatNamedEvents = false;

	/**
	 * Where to store the Unreal Insights Trace Files.
	 * If empty, Unreal Insights will connect to localhost (this computer) instead and you'll need to ensure Unreal Insights is launched.
	 * To specify a socket to connect to, use CommandLineArguments.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Unreal Insights", meta = (EditCondition="bEnableUnrealInsights"))
	FDirectoryPath ExplicitTraceFileSaveDirectory;

	/**
	 * A Console Variables Asset to always apply to your launches.
	 * All saved commands and variables will be executed before "Additional" ones specified below.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Console", meta = (AllowedClasses="/Script/ConsoleVariablesEditorRuntime.ConsoleVariablesAsset"))
	FSoftObjectPath ConsoleVariablesPreset;

	/**
	 * DPCvars
	 * You can specify additional console variables here to be executed after those of the Console Variable Preset are executed.
	 * This is useful for overriding variables in your defined Console Variables Asset.
	 * Console Variables generally have a name and a value, like "r.ScreenPercentage 50"
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Console", meta = (Keywords = "DPCvars"))
	TSet<FString> AdditionalConsoleVariables;

	/**
	 * You can specify additional console commands here to be executed after those of the Console Variable Preset are executed.
	 * This is useful for overriding commands in your defined Console Variables Asset.
	 * Console Commands generally have a name and no value (i.e. 'stat unit') though some have a value too (i.e. "r.SetNearClipPlane 500")
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Console")
	TSet<FString> AdditionalConsoleCommands;

	/**
	 * A list of command line arguments to append to the Switchboard command (e.g. messaging, fullscreen, handleensurepercent=0)
	 * Do not include the dash ("-") as this will be automatically added for you when calling the command.
	 * Parameters for arguments are supported, such as "handleensurepercent=0".
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Console")
	TSet<FString> CommandLineArguments;

	/**
	 * The name of the log file to which to write logs for the launched node.
	 * '.log' will be automatically appended to the file name.
	 * If not specified, the nDisplay node's name will be used instead.
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Logging")
	FString LogFileName;

	/**
	 * Which logs to include and with which verbosity level
	 */
	UPROPERTY(Config, EditAnywhere, Category="nDisplay Launch Settings|Logging")
	TArray<FDisplayClusterLaunchLoggingConstruct> Logging;
};
